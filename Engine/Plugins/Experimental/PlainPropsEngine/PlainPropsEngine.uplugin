{
	"FileVersion": 3,
	"FriendlyName": "PlainPropsEngine",
	"Description": "New Serialization Stack Prototype - Engine Bindings",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"IsExperimentalVersion": true,
	"SupportedTargetPlatforms": [
		"Win64"
	],
	"Modules": [
		{
			"Name": "PlainPropsEngine",
			"Type": "Runtime",
			"LoadingPhase": "PreLoadingScreen",
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins": [
		{
			"Name": "PlainProps",
			"Enabled": true
		},
		{
			"Name": "PlainPropsUObject",
			"Enabled": true
		},
		{
			"Name": "GameplayAbilities",
			"Enabled": true
		},
		{
			"Name": "Niagara",
			"Enabled": true
		}
	]
}
