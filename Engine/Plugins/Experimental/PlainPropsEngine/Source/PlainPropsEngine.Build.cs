// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PlainPropsEngine : ModuleRules
{
	public PlainPropsEngine(ReadOnlyTargetRules Target) : base(Target)
	{
		bDisableStaticAnalysis = true;

		CppStandard = CppStandardVersion.Cpp20;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"CorePreciseFP",
				"Engine",
				"GameplayAbilities",
				"Niagara",
				"PlainProps",
				"PlainPropsUObject"
		});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
			PrivateDependencyModuleNames.Add("BlueprintGraph");
		}
	}
}
