// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsEngineBindings.h"
#include "PlainPropsUObjectRuntime.h"
#include "PlainPropsRoundtripTest.h" // EBindMode
#include "PlainPropsBind.h"
#include "PlainPropsLoad.h"
#include "PlainPropsSave.h"
#include "PlainPropsInternalPrivateMemberPtr.h"
#include "Algo/IndexOf.h"
#include "Animation/AnimData/AttributeIdentifier.h"
#include "Animation/AttributeCurve.h"
#include "Animation/AttributeTypes.h"
#include "Misc/CoreMiscDefines.h"

// Temp hacks for non-intrusive prototype
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(FAnimationAttributeIdentifier_Private)
PP_DEFINE_PRIVATE_MEMBER_PTR(FAnimationAttributeIdentifier, Name,					FName);
PP_DEFINE_PRIVATE_MEMBER_PTR(FAnimationAttributeIdentifier, BoneName,				FName);
PP_DEFINE_PRIVATE_MEMBER_PTR(FAnimationAttributeIdentifier, BoneIndex,				int32);
PP_DEFINE_PRIVATE_MEMBER_PTR(FAnimationAttributeIdentifier, ScriptStruct,			TObjectPtr<UScriptStruct>);
PP_DEFINE_PRIVATE_MEMBER_PTR(FAnimationAttributeIdentifier, ScriptStructPath,		FSoftObjectPath);
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END()

PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(FAttributeCurve_Private)
PP_DEFINE_PRIVATE_MEMBER_PTR(FAttributeKey,				Value,						FWrappedAttribute);
PP_DEFINE_PRIVATE_MEMBER_PTR(FAttributeCurve,			Keys,						TArray<FAttributeKey>);	
PP_DEFINE_PRIVATE_MEMBER_PTR(FAttributeCurve,			ScriptStructPath,			FSoftObjectPath);
PP_DEFINE_PRIVATE_MEMBER_PTR(FAttributeCurve,			ScriptStruct,				TObjectPtr<UScriptStruct>);
PP_DEFINE_PRIVATE_MEMBER_PTR(FAttributeCurve,			bShouldInterpolate,			bool);
PP_DEFINE_PRIVATE_MEMBER_PTR(FAttributeCurve,			Operator,					const UE::Anim::IAttributeBlendOperator*);
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END()

namespace PlainProps::UE::Animation
{

static FScopeId GetAnimationScope()
{
	return GUE.Scopes.Engine;
}

struct FAnimationAttributeIdentifierBinding : ICustomBinding
{
	using Type = FAnimationAttributeIdentifier;
	using Ids = FRuntimeIds;

	const FMemberId MemberIds[4];
	
	FAnimationAttributeIdentifierBinding(TPropertySpecifier<4>& Spec)
	: MemberIds{Ids::IndexMember("Name"), Ids::IndexMember("BoneName"), Ids::IndexMember("BoneIndex"), Ids::IndexMember("ScriptStructPath")}
	{
		Spec.Members[0] = GUE.Structs.Name;
		Spec.Members[1] = GUE.Structs.Name;
		Spec.Members[2] = Specify<int32>();
		Spec.Members[3] = GUE.Structs.SoftObjectPath;
	}

	void Save(FMemberBuilder& Dst, const FAnimationAttributeIdentifier& Src, const FAnimationAttributeIdentifier* Default, const FSaveContext& Ctx) const
	{
		using namespace FAnimationAttributeIdentifier_Private;

		Dst.AddStruct(MemberIds[0], GUE.Structs.Name, SaveStruct(&(Src.*_Name), GUE.Structs.Name, Ctx));
		Dst.AddStruct(MemberIds[1], GUE.Structs.Name, SaveStruct(&(Src.*_BoneName), GUE.Structs.Name, Ctx));
		Dst.Add(MemberIds[2], Src.*_BoneIndex);
		Dst.AddStruct(MemberIds[3], GUE.Structs.SoftObjectPath, SaveStruct(&(Src.*_ScriptStructPath), GUE.Structs.SoftObjectPath, Ctx));
	}

	void Load(FAnimationAttributeIdentifier& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
	{
		using namespace FAnimationAttributeIdentifier_Private;

		FMemberLoader It(Src);
		LoadStruct(&(Dst.*_Name), It.GrabStruct());
		LoadStruct(&(Dst.*_BoneName), It.GrabStruct());
		Dst.*_BoneIndex = It.GrabLeaf().As<int32>();
		LoadStruct(&(Dst.*_ScriptStructPath), It.GrabStruct());
		Dst.*_ScriptStruct = Cast<UScriptStruct>((Dst.*_ScriptStructPath).ResolveObject());
	}

	inline static bool Diff(const FAnimationAttributeIdentifier& A, const FAnimationAttributeIdentifier& B, const FBindContext&)
	{
		return A != B;
	}
};

struct FAttributeCurveBinding : ICustomBinding
{
	using Type = FAttributeCurve;
	using Ids = FRuntimeIds;

	const FDualStructId AttributeKeyId;
	const FMemberId MemberIds[3];
	
	FAttributeCurveBinding(TPropertySpecifier<3>& Spec)
	: AttributeKeyId(GUE.DynamicIds.GetStruct(FAttributeKey::StaticStruct()))
	, MemberIds{Ids::IndexMember("Keys"), Ids::IndexMember("ScriptStructPath"), Ids::IndexMember("DynamicKeys")}
	{
		Spec.Members[0] = DefaultRangeOf(AttributeKeyId);
		Spec.Members[1] = GUE.Structs.SoftObjectPath;
		Spec.Members[2] = DefaultRangeOf(SpecDynamicStruct);
	}

	static FTypedRange SaveKeysStatic(FBindId Id, TConstArrayView<FAttributeKey> Keys, const FSaveContext& Ctx)
	{
		FStructRangeSaver Out(Ctx.Scratch, Keys.Num());
		for (const FAttributeKey& Key : Keys)
		{
			Out.AddItem(SaveStruct(&Key, Id, Ctx));
		}
		return Out.Finalize(MakeStructRangeSchema(DefaultRangeMax, Id));
	}

	static FTypedRange SaveKeysDynamic(FBindId Id, TConstArrayView<FAttributeKey> Keys, const FSaveContext& Ctx)
	{
		using namespace FAttributeCurve_Private;

		FStructRangeSaver Out(Ctx.Scratch, Keys.Num());
		for (const FAttributeKey& Key : Keys)
		{
			Out.AddItem(SaveStruct((Key.*_Value).GetPtr<void>(), Id, Ctx));
		}
		return Out.Finalize(MakeStructRangeSchema(DefaultRangeMax, Id));
	}

	void Save(FMemberBuilder& Dst, const FAttributeCurve& Src, const FAttributeCurve* Default, const FSaveContext& Ctx) const
	{
		using namespace FAttributeCurve_Private;

		Dst.AddRange(MemberIds[0], SaveKeysStatic(AttributeKeyId, Src.*_Keys, Ctx));
		Dst.AddStruct(MemberIds[1], GUE.Structs.SoftObjectPath, SaveStruct(&(Src.*_ScriptStructPath), GUE.Structs.SoftObjectPath, Ctx));
		FBindId DynamicKeyId = GUE.DynamicIds.GetStruct(Src.*_ScriptStruct);
		Dst.AddRange(MemberIds[2], SaveKeysDynamic(DynamicKeyId, Src.*_Keys, Ctx));
	}

	static void LoadKeysStatic(TArray<FAttributeKey>& Dst, FStructRangeLoadView Src)
	{
		Dst.SetNum(static_cast<int32>(Src.Num()));
		FAttributeKey* DstIt = Dst.GetData();
		for (FStructLoadView SrcIt : Src)
		{
			LoadStruct(DstIt++, SrcIt);
		}
	}

	static void LoadKeysDynamic(UScriptStruct* ScriptStruct, TArray<FAttributeKey>& Dst, FStructRangeLoadView Src)
	{
		using namespace FAttributeCurve_Private;

		check(Dst.Num() == Src.Num());
		FAttributeKey* DstIt = Dst.GetData();
		for (FStructLoadView SrcIt : Src)
		{
			FAttributeKey& Item = *DstIt++;
			(Item.*_Value).Allocate(ScriptStruct);
			LoadStruct((Item.*_Value).GetPtr<void>(), SrcIt);
		}
	}

	void Load(FAttributeCurve& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
	{
		using namespace FAttributeCurve_Private;

		FMemberLoader It(Src);
		LoadKeysStatic(Dst.*_Keys, It.GrabRange().AsStructs());
		LoadStruct(&(Dst.*_ScriptStructPath), It.GrabStruct());
		UScriptStruct* ScriptStruct = Cast<UScriptStruct>((Dst.*_ScriptStructPath).ResolveObject());
		LoadKeysDynamic(ScriptStruct, Dst.*_Keys, It.GrabRange().AsStructs());
		Dst.*_ScriptStruct = ScriptStruct;
		Dst.*_Operator = ::UE::Anim::AttributeTypes::GetTypeOperator(ScriptStruct);
		Dst.*_bShouldInterpolate = ::UE::Anim::AttributeTypes::CanInterpolateType(ScriptStruct);
	}

	inline static bool Diff(const FAttributeCurve& A, const FAttributeCurve& B, const FBindContext&)
	{
		using namespace FAttributeCurve_Private;

		if (A.*_ScriptStructPath != B.*_ScriptStructPath)
		{
			return true;
		}
		check(A.*_ScriptStruct == B.*_ScriptStruct);
		if (A.*_Operator != B.*_Operator)
		{
			return true;
		}
		if (A.*_bShouldInterpolate != B.*_bShouldInterpolate)
		{
			return true;
		}
		const TArray<FAttributeKey>& KeysA = A.*_Keys;
		const TArray<FAttributeKey>& KeysB = B.*_Keys;
		if (KeysA.Num() != KeysB.Num())
		{
			return true;
		}
		UScriptStruct* ScriptStruct = A.*_ScriptStruct;
		for (int32 I = 0, Num = KeysA.Num(); I < Num; ++I)
		{
			// compare the reflected properties
			if (!FAttributeKey::StaticStruct()->CompareScriptStruct(&KeysA[I], &KeysB[I], PPF_None))
			{
				return true;
			}
			// compare the non-reflected dynamic struct properties
			if (!ScriptStruct->CompareScriptStruct((KeysA[I].*_Value).GetPtr<void>(), (KeysB[I].*_Value).GetPtr<void>(), PPF_None))
			{
				return true;
			}
		}
		return false;
	}
};

} // namespace PlainProps::UE::Animation

////////////////////////////////////////////////////////////////////////////////////////////////

namespace PlainProps::UE
{
void CustomBindAnimationTypes(EBindMode Mode)
{
	using namespace Animation;
	// Todo: Ownership / memory leak
	new TScopedDefaultUStructBinding<FAnimationAttributeIdentifier, FAnimationAttributeIdentifierBinding>();
	new TScopedDefaultUStructBinding<FAttributeCurve, FAttributeCurveBinding>();
}
} // namespace PlainProps::UE

////////////////////////////////////////////////////////////////////////////////////////////////

namespace PlainProps
{
template<> struct TOccupancyOf<FAnimationAttributeIdentifier> : FRequireAll {};
template<> struct TOccupancyOf<FAttributeKey> : FRequireAll {};
template<> struct TOccupancyOf<FAttributeCurve> : FRequireAll {};
}  // namespace PlainProps
