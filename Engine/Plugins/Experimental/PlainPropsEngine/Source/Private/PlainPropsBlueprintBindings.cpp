// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsEngineBindings.h"
#include "PlainPropsUObjectRuntime.h"
#include "PlainPropsUObjectMetaBindingsInternal.h" // Temp for UE_CUSTOM_BIND_MEMBERS and BindStruct()
#include "PlainPropsRoundtripTest.h" // EBindMode
#include "PlainPropsBind.h"
#include "PlainPropsLoad.h"
#include "PlainPropsSave.h"
#include "PlainPropsInternalPrivateMemberPtr.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#if WITH_EDITOR
#include "K2Node_EditablePinBase.h"
#endif

// Temp hacks for non-intrusive prototype
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(EdGraphPin_Private)
PP_DEFINE_PRIVATE_MEMBER_PTR(FEdGraphPinReference,	OwningNode,	TWeakObjectPtr<UEdGraphNode>);
PP_DEFINE_PRIVATE_MEMBER_PTR(FEdGraphPinReference,	PinId,		FGuid);
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END()

#if WITH_EDITORONLY_DATA
PP_REFLECT_STRUCT(, UEdGraphPin, void, PinId, PinName, PinFriendlyName, SourceIndex, PinToolTip, Direction, PinType, DefaultValue, AutogeneratedDefaultValue, DefaultObject, DefaultTextValue, PersistentGuid);
#else
PP_REFLECT_STRUCT(, UEdGraphPin, void, PinId, PinName,                  SourceIndex, PinToolTip, Direction, PinType, DefaultValue, AutogeneratedDefaultValue, DefaultObject, DefaultTextValue);
#endif


namespace PlainProps::UE::Blueprint
{

static FScopeId GetBlueprintScope()
{
	return GUE.Scopes.Engine;
}

// struct id for nested ctti schema binding for UEdGraphPin (not-a-UClass)
static FDualStructId IndexSchemaEdGraphPin()
{
	FType Type =
	{
		IndexScope<FRuntimeIds, TTypename<UEdGraphPin>>(),
		FTypenameId(FRuntimeIds::IndexTypename("UEdGraphPin.Schema"))
	};
	FStructId Id = FRuntimeIds::IndexStruct(Type);
	return FDualStructId(Id);
}

// struct id for nested UE reflected schema binding for UEdGraphNode (is-a-UClass)
static FDualStructId IndexSchemaEdGraphNode()
{
	FType Type = { GetBlueprintScope(), FTypenameId(FRuntimeIds::IndexTypename("EdGraphNode.Schema")) };
	FStructId Id = FRuntimeIds::IndexStruct(Type);
	return FDualStructId(Id);
}

////////////////////////////////////////////////////////////////////////////////////////////////

struct UEdGraphNodeBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(UEdGraphNode, Schema, Pins);
	UEdGraphNodeBinding(TPropertySpecifier<NumMembers>& Spec);
	
	const FDualStructId SchemaId;
	const FDualStructId EdGraphPinId;

	void Save(FMemberBuilder& Dst, const UEdGraphNode& Src, const UEdGraphNode* Default, const FSaveContext& Ctx) const;
	void Load(UEdGraphNode& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool Diff(const UEdGraphNode& A, const UEdGraphNode& B, const FBindContext&) { return &A != &B; }

	static FTypedRange SavePinArray(FBindId Id, TConstArrayView<UEdGraphPin*> Pins, const FSaveContext& Ctx);
	static void LoadPinArray(UEdGraphNode& Dst, FStructRangeLoadView Src);

};

UEdGraphNodeBinding::UEdGraphNodeBinding(TCustomSpecifier<Ids, NumMembers>& Spec)
: SchemaId(IndexSchemaEdGraphNode())
, EdGraphPinId(GetStructDualId<Ids, UEdGraphPin>())
{
	Spec.Members[M::Schema] = FDeclId(SchemaId);
	Spec.Members[M::Pins] = DefaultRangeOf(FDeclId(EdGraphPinId));
}

void UEdGraphNodeBinding::Save(FMemberBuilder& Dst, const UEdGraphNode& Src, const UEdGraphNode* Default, const FSaveContext& Ctx) const
{
	// Super::Serialize(Ar);
	Dst.AddStruct(MemberIds[M::Schema], FBindId(SchemaId), SaveStruct(&Src, FBindId(SchemaId), Ctx));
	if (Src.Pins.Num())
	{
		// see UEdGraphPin::SerializeAsOwningNode()/UEdGraphPin::SerializePinArray()
		Dst.AddRange(MemberIds[M::Pins], SavePinArray(FBindId(EdGraphPinId), Src.Pins, Ctx));
	}
}

void UEdGraphNodeBinding::Load(UEdGraphNode& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	FMemberLoader Members(Src);
	LoadStruct(&Dst, Members.GrabStruct()); // grab schema data
	if (Members.HasMore())
	{
		LoadPinArray(Dst, Members.GrabRange().AsStructs());
	}
}

FTypedRange UEdGraphNodeBinding::SavePinArray(FBindId Id, TConstArrayView<UEdGraphPin*> Pins, const FSaveContext& Ctx)
{
	FStructRangeSaver Out(Ctx.Scratch, Pins.Num());
	for (const UEdGraphPin* Pin : Pins)
	{
		check(Pin);
		Out.AddItem(SaveStruct(Pin, Id, Ctx));
	}
	return Out.Finalize(MakeStructRangeSchema(DefaultRangeMax, Id));
}

void UEdGraphNodeBinding::LoadPinArray(UEdGraphNode& Dst, FStructRangeLoadView Src)
{
	check(Dst.Pins.IsEmpty());
	for (FStructLoadView SrcIt : Src)
	{
		UEdGraphPin* Pin = UEdGraphPin::CreatePin(&Dst);
		LoadStruct(Pin, SrcIt);
		Dst.Pins.Add(Pin);
#if PP_OVERRIDE_OBJECT_LOADING
		UEdGraphPin::ResolveReferencesToPin(Pin, /*bStrictValidation*/ true);
#endif
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

struct UEdGraphPinBinding : ICustomBinding
{
#if WITH_EDITORONLY_DATA
	UE_CUSTOM_BIND_MEMBERS(UEdGraphPin,
		Schema, ParentPin, ReferencePassThroughConnection, LinkedTo, SubPins,
		bHidden, bNotConnectable, bDefaultValueIsReadOnly, bDefaultValueIsIgnored,
		bAdvancedView, bOrphanedPin, bHasSnappedChild, bHasSnappedParent);
#else
	UE_CUSTOM_BIND_MEMBERS(UEdGraphPin,
		Schema, ParentPin, ReferencePassThroughConnection, LinkedTo, SubPins);
#endif
	UEdGraphPinBinding(TCustomSpecifier<Ids, NumMembers>& Spec);
	
	const FDualStructId SchemaId;
	const FDualStructId PinReferenceId;

	void Save(FMemberBuilder& Dst, const UEdGraphPin& Src, const UEdGraphPin* Default, const FSaveContext& Ctx) const;
	void Load(UEdGraphPin& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool Diff(const UEdGraphPin& A, const UEdGraphPin& B, const FBindContext&) { return &A != &B; }

	static FTypedRange SavePinReferenceArray(FBindId Id, TConstArrayView<UEdGraphPin*> Pins, const FSaveContext& Ctx);
	static void LoadPinReferenceArray(UEdGraphPin& RequestingPin, TArray<UEdGraphPin*>& Pins, FStructRangeLoadView Src, EPinResolveType ResolveType);
	static void LoadPinReference(UEdGraphPin& RequestingPin, UEdGraphPin*& PinRef, FStructLoadView Struct, EPinResolveType ResolveType, int32 ArrayIdx = INDEX_NONE);

};

UEdGraphPinBinding::UEdGraphPinBinding(TCustomSpecifier<Ids, NumMembers>& Spec)
: SchemaId(IndexSchemaEdGraphPin())
, PinReferenceId(GUE.DynamicIds.GetStruct(FEdGraphPinReference::StaticStruct()))
{
	Spec.Members[M::Schema] = FDeclId(SchemaId);
	Spec.Members[M::ParentPin] = FDeclId(PinReferenceId);
	Spec.Members[M::ReferencePassThroughConnection] = FDeclId(PinReferenceId);
	Spec.Members[M::LinkedTo] = DefaultRangeOf(FDeclId(PinReferenceId));
	Spec.Members[M::SubPins] = DefaultRangeOf(FDeclId(PinReferenceId));
#if WITH_EDITORONLY_DATA
	Spec.Members[M::bHidden] = Specify<bool>();
	Spec.Members[M::bNotConnectable] = Specify<bool>();
	Spec.Members[M::bDefaultValueIsReadOnly] = Specify<bool>();
	Spec.Members[M::bDefaultValueIsIgnored] = Specify<bool>();
	Spec.Members[M::bAdvancedView] = Specify<bool>();
	Spec.Members[M::bOrphanedPin] = Specify<bool>();
	Spec.Members[M::bHasSnappedChild] = Specify<bool>();
	Spec.Members[M::bHasSnappedParent] = Specify<bool>();
#endif
}

void UEdGraphPinBinding::Save(FMemberBuilder& Dst, const UEdGraphPin& Src, const UEdGraphPin* Default, const FSaveContext& Ctx) const
{
#if PP_OVERRIDE_OBJECT_LOADING
	Dst.AddStruct(MemberIds[M::Schema], FBindId(SchemaId), SaveStruct(&Src, FBindId(SchemaId), Ctx));
	if (Src.ParentPin)
	{
		FEdGraphPinReference PinRef(Src.ParentPin);
		Dst.AddStruct(MemberIds[M::ParentPin], FBindId(PinReferenceId), SaveStruct(&PinRef, FBindId(PinReferenceId), Ctx));
	}
	if (Src.ReferencePassThroughConnection)
	{
		FEdGraphPinReference PinRef(Src.ReferencePassThroughConnection);
		Dst.AddStruct(MemberIds[M::ReferencePassThroughConnection], FBindId(PinReferenceId), SaveStruct(&PinRef, FBindId(PinReferenceId), Ctx));
	}
	if (Src.LinkedTo.Num())
	{
		// see UEdGraphPin::SerializePinArray()
		Dst.AddRange(MemberIds[M::LinkedTo], SavePinReferenceArray(FBindId(PinReferenceId), Src.LinkedTo, Ctx));
	}
	if (Src.SubPins.Num())
	{
		// see UEdGraphPin::SerializePinArray()
		Dst.AddRange(MemberIds[M::SubPins], SavePinReferenceArray(FBindId(PinReferenceId), Src.SubPins, Ctx));
	}
#if WITH_EDITORONLY_DATA
	Dst.Add(MemberIds[M::bHidden], (bool)Src.bHidden);
	Dst.Add(MemberIds[M::bNotConnectable], (bool)Src.bNotConnectable);
	Dst.Add(MemberIds[M::bDefaultValueIsReadOnly], (bool)Src.bDefaultValueIsReadOnly);
	Dst.Add(MemberIds[M::bDefaultValueIsIgnored], (bool)Src.bDefaultValueIsIgnored);
	Dst.Add(MemberIds[M::bAdvancedView], (bool)Src.bAdvancedView);
	Dst.Add(MemberIds[M::bOrphanedPin], (bool)Src.bOrphanedPin);
	Dst.Add(MemberIds[M::bHasSnappedChild], Src.HasSnappedChild());
	Dst.Add(MemberIds[M::bHasSnappedParent], Src.HasSnappedParent());
#endif
#endif
}

void UEdGraphPinBinding::Load(UEdGraphPin& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
#if PP_OVERRIDE_OBJECT_LOADING
	FMemberLoader Members(Src);
	LoadStruct(&Dst, Members.GrabStruct()); // grab schema data
	if (Members.HasMore() && Members.PeekName() == MemberIds[M::ParentPin])
	{
		LoadPinReference(Dst, Dst.ParentPin, Members.GrabStruct(), EPinResolveType::ParentPin);
	}
	if (Members.HasMore() && Members.PeekName() == MemberIds[M::ReferencePassThroughConnection])
	{
		LoadPinReference(Dst, Dst.ReferencePassThroughConnection, Members.GrabStruct(), EPinResolveType::ReferencePassThroughConnection);
	}
	if (Members.HasMore() && Members.PeekName() == MemberIds[M::LinkedTo])
	{
		LoadPinReferenceArray(Dst, Dst.LinkedTo, Members.GrabRange().AsStructs(), EPinResolveType::LinkedTo);
	}
	if (Members.HasMore() && Members.PeekName() == MemberIds[M::SubPins])
	{
		LoadPinReferenceArray(Dst, Dst.SubPins, Members.GrabRange().AsStructs(), EPinResolveType::SubPins);
	}
#if WITH_EDITORONLY_DATA
	Dst.bHidden = Members.GrabLeaf().AsBool();
	Dst.bNotConnectable = Members.GrabLeaf().AsBool();
	Dst.bDefaultValueIsReadOnly = Members.GrabLeaf().AsBool();
	Dst.bDefaultValueIsIgnored = Members.GrabLeaf().AsBool();
	Dst.bAdvancedView = Members.GrabLeaf().AsBool();
	Dst.bOrphanedPin = Members.GrabLeaf().AsBool();
	Dst.SetHasSnappedChild(Members.GrabLeaf().AsBool());
	Dst.SetHasSnappedParent(Members.GrabLeaf().AsBool());
#endif
#endif
}

FTypedRange UEdGraphPinBinding::SavePinReferenceArray(FBindId Id, TConstArrayView<UEdGraphPin*> Pins, const FSaveContext& Ctx)
{
	FStructRangeSaver Out(Ctx.Scratch, Pins.Num());
	for (const UEdGraphPin* Pin : Pins)
	{
		FEdGraphPinReference PinRef(Pin);
		Out.AddItem(SaveStruct(&PinRef, Id, Ctx));
	}
	return Out.Finalize(MakeStructRangeSchema(DefaultRangeMax, Id));
}

void UEdGraphPinBinding::LoadPinReferenceArray(UEdGraphPin& RequestingPin, TArray<UEdGraphPin*>& Pins, FStructRangeLoadView Src, EPinResolveType ResolveType)
{
	check(Src.Num());
	check(Pins.IsEmpty());
	Pins.SetNum(Src.Num());
	int32 ArrayIdx = 0;
	for (FStructLoadView SrcIt : Src)
	{
		LoadPinReference(RequestingPin, Pins[ArrayIdx], SrcIt, ResolveType, ArrayIdx);
		++ArrayIdx;
	}
}

void UEdGraphPinBinding::LoadPinReference(UEdGraphPin& RequestingPin, UEdGraphPin*& PinRef, FStructLoadView Struct, EPinResolveType ResolveType, int32 ArrayIdx)
{
	using namespace EdGraphPin_Private;

	FEdGraphPinReference Temp;
	LoadStruct(&Temp, Struct);
	if (UEdGraphPin* Pin = Temp.Get())
	{
		PinRef = Pin;
	}
	else
	{
		checkf(!PinRef, TEXT("Interesting. The Pin already has a value when the loaded ref resolve is deferred."));
#if PP_OVERRIDE_OBJECT_LOADING
		UEdGraphPin::AddUnresolvedPin(RequestingPin, Temp.*_OwningNode, Temp.*_PinId, ResolveType, ArrayIdx);
#endif
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR

struct UK2Node_EditablePinBaseBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(UK2Node_EditablePinBase, bIsEditable, UserDefinedPins);
	UK2Node_EditablePinBaseBinding(TCustomSpecifier<Ids, NumMembers>& Spec);

	const FDualStructId EdGraphNodeId;
	const FDualStructId UserPinInfoId;

	void Save(FMemberBuilder& Dst, const UK2Node_EditablePinBase& Src, const UK2Node_EditablePinBase* Default, const FSaveContext& Ctx) const;
	void Load(UK2Node_EditablePinBase& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool Diff(const UK2Node_EditablePinBase& A, const UK2Node_EditablePinBase& B, const FBindContext&) { return &A != &B; }
};

UK2Node_EditablePinBaseBinding::UK2Node_EditablePinBaseBinding(TCustomSpecifier<Ids, NumMembers>& Spec)
: EdGraphNodeId(GUE.DynamicIds.GetStruct(UEdGraphNode::StaticClass()))
, UserPinInfoId(GUE.DynamicIds.GetStruct(FUserPinInfo::StaticStruct()))
{
	Spec.Super = FDeclId(EdGraphNodeId);
	Spec.Members[M::bIsEditable] = Specify<bool>();
	Spec.Members[M::UserDefinedPins] = DefaultRangeOf(FDeclId(UserPinInfoId));
}

void UK2Node_EditablePinBaseBinding::Save(FMemberBuilder& Dst, const UK2Node_EditablePinBase& Src, const UK2Node_EditablePinBase* Default, const FSaveContext& Ctx) const
{
	SaveSuper(Dst, &Src, FBindId(EdGraphNodeId), Ctx);
	Dst.Add(MemberIds[M::bIsEditable], (bool)Src.bIsEditable);
	if (Src.UserDefinedPins.Num())
	{
		FStructRangeSaver Out(Ctx.Scratch, Src.UserDefinedPins.Num());
		for (const TSharedPtr<FUserPinInfo>& PinInfo : Src.UserDefinedPins)
		{
			check(PinInfo.IsValid());
			Out.AddItem(SaveStruct(PinInfo.Get(), FBindId(UserPinInfoId), Ctx));
		}
		Dst.AddRange(MemberIds[M::UserDefinedPins], Out.Finalize(MakeStructRangeSchema(DefaultRangeMax, FBindId(UserPinInfoId))));
	}
}

void UK2Node_EditablePinBaseBinding::Load(UK2Node_EditablePinBase& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	FMemberLoader Members(Src);
	TryLoadSuper(&Dst, Members);
	Dst.bIsEditable = Members.GrabLeaf().AsBool();
	if (Members.HasMore())
	{
		for (FStructLoadView SrcIt : Members.GrabRange().AsStructs())
		{
			TSharedPtr<FUserPinInfo> PinInfo = MakeShared<FUserPinInfo>();
			LoadStruct(PinInfo.Get(), SrcIt);
			Dst.UserDefinedPins.Add(PinInfo);
		}
	}
}

#endif // WITH_EDITOR

} // namespace PlainProps::UE::Blueprint

////////////////////////////////////////////////////////////////////////////////////////////////

namespace PlainProps::UE
{
void CustomBindBlueprintTypes(EBindMode Mode)
{
#if PP_OVERRIDE_OBJECT_LOADING
	using namespace Blueprint;
	// temp: create a custom UEdGraphPin (not-a-UClass) binding with a nested UEdGraphPin.Schema ctti binding
	new TScopedStructBinding<UEdGraphPin, FDefaultRuntime, void>(IndexSchemaEdGraphPin());
	new TScopedStructBinding<UEdGraphPin, FDefaultRuntime, UEdGraphPinBinding>();
	// temp: create a custom UEdGraphNode (is-a-UClass) binding with a nested /Script/Engine.EdGraphNode.Schema reflected property binding
	BindStruct(FBindId(IndexSchemaEdGraphNode()), UEdGraphNode::StaticClass());
	new TScopedDefaultUClassBinding<UEdGraphNode, UEdGraphNodeBinding>();
#if WITH_EDITOR
	new TScopedDefaultUClassBinding<UK2Node_EditablePinBase, UK2Node_EditablePinBaseBinding>();
#endif
#endif
}
} // namespace PlainProps::UE

namespace PlainProps
{
template<> struct TTypename<EEdGraphPinDirection>
{ inline static constexpr std::string_view DeclName = "EEdGraphPinDirection"; };
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<EEdGraphPinDirection>>()
{ return UE::Blueprint::GetBlueprintScope(); }

// Enable usage of FEdGraphPinType as a member in ctti reflected structs like UEdGraphPin 
template<> struct TTypename<FEdGraphPinType>
{ inline static constexpr std::string_view DeclName = "EdGraphPinType"; };
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FEdGraphPinType>>()
{ return UE::Blueprint::GetBlueprintScope(); }
} // namespace PlainProps

