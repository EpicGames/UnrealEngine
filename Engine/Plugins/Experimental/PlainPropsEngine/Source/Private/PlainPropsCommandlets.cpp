// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsCommandlets.h"
#include "PlainPropsEngineBindings.h"
#include "PlainPropsRoundtripTest.h"
#include "PlainPropsTestBindings.h"
#include "PlainPropsTestAssets.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/World.h"
#include "Misc/AsciiSet.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Misc/PackageName.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "UObject/PackageReload.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "AssetCompilingManager.h"
#include "Engine/AssetManager.h"
#include "Engine/Blueprint.h"
#include "Kismet2/KismetEditorUtilities.h"
#endif

// Temporary maintained filter to ignore diffs reported by RoundtripViaLinkerBatch
// that currently relies on IsIdentical property comparisons of loaded objects.
// This will be replaced with a more robust and scalable approach that also captures custom bound data.
static PlainProps::UE::FLinkerDiffFilter LinkerDiffFilter
{
	.BypassNativeIdenticalStructs
	{
		"FontData",							// Native FFontData::operator== with object pointers
		"GameplayEffectVersion",			// FGameplayEffectVersion::Identical is hardcoded to return false
		"NiagaraMeshMaterialOverride",		// Native FNiagaraMeshMaterialOverride::operator== with object pointers
	},
	.IgnoreStructs
	{
		// Runtime struct ignore diff list
		"NiagaraCompileHash",				// FNiagaraCompileHash has just a single TArray<uint8> DataHash property

		// Missing struct bindings
		"InstancedPropertyBag",				// FInstancedPropertyBag in CoreUObject
		"MovieSceneFrameRange",				// FMovieSceneFrameRange in MovieScene,
											// requires Math bindings for TRange<T> and TRangeBound<T>
		"ShaderValueTypeHandle",			// FShaderValueTypeHandle in ComputeFramework
		"Spline",							// FSpline in Engine,
											// requires GeometryCore binding for TMultiSpline
		"UniversalObjectLocator",			// FUniversalObjectLocator in Engine,
											// requires Engine binding for FUniversalObjectLocatorFragment
	},
	.IgnorePropertiesForStructs
	{
		TPair<FName, FName>("BaseId",							"NiagaraGraphScriptUsageInfo"		),
		TPair<FName, FName>("ChangeId",							"NiagaraNode"						),
		TPair<FName, FName>("ChangeId",							"NiagaraGraph"						),
		TPair<FName, FName>("LastBuiltTraversalDataChangeId",	"NiagaraGraph"						),
		TPair<FName, FName>("LayerUsageDebugColor",				"LandscapeLayerInfoObject"			),
		TPair<FName, FName>("UniqueId",							"NiagaraParameterDefinitionsBase"	),
		TPair<FName, FName>("Version",							"NiagaraDataChannelVariable"		),
	},
	.IgnorePropertiesForBases
	{
		{"GraphGuid",	"EdGraph"},
		{"NodeGuid",	"EdGraphNode"},
	},
	.IgnoreCastFlags = CASTCLASS_FMulticastInlineDelegateProperty | CASTCLASS_FMulticastSparseDelegateProperty,
};

static void FlushAsyncOperations(const TCHAR* Context)
{
	UE_LOGFMT(LogPlainPropsEngine, Display, "FlushAsyncOperations : {Context} ...", Context);
	
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(Context);
	FlushAsyncLoading();
	(*GFlushStreamingFunc)();
#if WITH_EDITOR
	FAssetCompilingManager::Get().FinishAllCompilation();
#endif
}

static FName GenerateDroppedPackageName(FName Name, int32 NewNumber = INDEX_NONE)
{
	if (NewNumber == INDEX_NONE)
	{
		return MakeUniqueObjectName(nullptr, UPackage::StaticClass(), NAME_TrashedPackage);
	}
	// Transform /Game/Package_01 which is represented as "/Game/Package" and number 1 into being
	// represented as "/Game/Package_01" and number 0, so that number 0 can be replace by the new number.
	// This works for packages named: /Game/Package_01, /Game/Package_02, ..., /Game/Package_10
	if (Name.GetNumber())
	{
		Name = FName(*Name.ToString(), 0);
	}
	return FName(Name, NewNumber + 1);
}

// Code inspired by UPackageTools::ReloadPackages
static void DropPackages(TArray<UPackage*>&& Packages, int32 NewNumber)
{
	FlushAsyncOperations(TEXT("DropPackages"));

	TArray<UObject*> AllObjects;
	TArray<UObject*> ObjectsInPackage;
	TArray<FName> NewNames;

	SortPackagesForReload(Packages);
	NewNames.Reserve(Packages.Num());
	AllObjects.Reserve(10*Packages.Num());
	for (UPackage* Package : Packages)
	{
		GetObjectsWithPackage(Package, ObjectsInPackage, EGetObjectsFlags::IncludeNestedObjects, RF_Transient, EInternalObjectFlags::Garbage);
		AllObjects.Append(ObjectsInPackage);
		FName NewName = GenerateDroppedPackageName(Package->GetFName(), NewNumber);
		NewNames.Add(NewName);
		UE_LOGFMT(LogPlainPropsEngine, Log, " Dropping package {Package} with {Num} objects => {NewName}",
			Package->GetFName(), ObjectsInPackage.Num(), NewName);
		ObjectsInPackage.Reset();
	}

	FCoreDelegates::CleanupUnloadingObjects.Broadcast(AllObjects);

#if WITH_EDITOR
	for (UObject* Obj : AllObjects)
	{
		// Asset manager can hold hard references to this object and prevent GC
		{
			const FPrimaryAssetId PrimaryAssetId = UAssetManager::Get().GetPrimaryAssetIdForObject(Obj);
			if (PrimaryAssetId.IsValid())
			{
				UAssetManager::Get().UnloadPrimaryAsset(PrimaryAssetId);
			}
		}

		if (UBlueprint* BP = Cast<UBlueprint>(Obj))
		{
			BP->ClearEditorReferences();

			// Remove from cached dependent lists.
			for (const TWeakObjectPtr<UBlueprint> Dependency : BP->CachedDependencies)
			{
				if (UBlueprint* ResolvedDependency = Dependency.Get())
				{
					ResolvedDependency->CachedDependents.Remove(BP);
				}
			}

			BP->CachedDependencies.Reset();

			// Remove from cached dependency lists.
			for (const TWeakObjectPtr<UBlueprint> Dependent : BP->CachedDependents)
			{
				if (UBlueprint* ResolvedDependent = Dependent.Get())
				{
					ResolvedDependent->CachedDependencies.Remove(BP);
				}
			}

			BP->CachedDependents.Reset();
		}
		else if (UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(Obj))
		{
			FKismetEditorUtilities::OnBlueprintGeneratedClassUnloaded.Broadcast(BPGC);
		}
		// else if (UWorld* World = Cast<UWorld>(Obj))
		// {
		// 	if (World->bIsWorldInitialized)
		// 	{
		// 		World->CleanupWorld();
		// 	}
		// }
	}
#endif

	for (UObject* Obj : AllObjects)
	{
		Obj->SetFlags(RF_NewerVersionExists);
	}

	for (int32 I = 0; I < Packages.Num(); ++I)
	{
		Packages[I]->Rename(*NewNames[I].ToString(), nullptr, REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty);
		Packages[I]->SetFlags(RF_NewerVersionExists);
	}
}

// ALevelEntity::PreSaveFromRoot called during save requires a verse::level_entity
// that is created from PreRegisterAllComponents.
static void InitWorlds()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(InitWorlds);

	// Code inspired by UCookOnTheFlyServer::ValidateSourcePackage
	FWorldInitializationValues IVS;
	IVS.AllowAudioPlayback(false);
	IVS.RequiresHitProxies(false);
	IVS.ShouldSimulatePhysics(false);
	IVS.EnableTraceCollision(true);
	IVS.SetTransactional(false);
	IVS.CreateWorldPartition(true);
	IVS.CreateAISystem(false);
	IVS.CreateNavigation(false);

	for (TObjectIterator<UWorld> It; It; ++It)
	{
		if (!It->bIsWorldInitialized)
		{
			UE_LOGFMT(LogPlainPropsEngine, Log, "Init world {World}...", It->GetFName());
			It->InitWorld(IVS);
			It->UpdateWorldComponents(true, true);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

UTestPlainPropsCommandlet::UTestPlainPropsCommandlet(const FObjectInitializer& Init)
: Super(Init)
{}

int32 UTestPlainPropsCommandlet::Main(const FString& Params)
{
	using namespace PlainProps;
	using namespace PlainProps::UE;

	TArray<UObject*> Objects;
	if (int32 LoadIdx = Params.Find("-load="); LoadIdx != INDEX_NONE)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(LoadAssets);
		// E.g.
		// -load=/BRRoot/BRRoot.BRRoot,/Game/Maps/FrontEnd
		// -load=maps
		// -load=all

		const TCHAR* It = &Params[LoadIdx + 6];
		FStringView AssetString(It, FAsciiSet::FindFirstOrEnd(It, FAsciiSet(" ")) - It);
		UE_LOGFMT(LogPlainPropsEngine, Display, "Loading {AssetString}...", AssetString);

		const bool bLoadMaps = AssetString == TEXT("maps");
		if (bLoadMaps || AssetString == TEXT("all"))
		{
			UE_LOGFMT(LogPlainPropsEngine, Display, "Loading asset registry...");
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName);
			AssetRegistryModule.Get().SearchAllAssets(true);

			TArray<FAssetData> Assets;

			if (bLoadMaps)
			{
				// find all .umaps in asset registry
				UE_LOGFMT(LogPlainPropsEngine, Display, "Gathering all maps...");
				AssetRegistryModule.Get().GetAssetsByClass(FTopLevelAssetPath("/Script/Engine", "World"), /* out */ Assets, true);
				UE_LOGFMT(LogPlainPropsEngine, Display, "Loading all {Assets} maps...", Assets.Num());
			}
			else
			{
				// find all .uassets and .umaps in asset registry
				UE_LOGFMT(LogPlainPropsEngine, Display, "Gathering all assets...");
				AssetRegistryModule.Get().GetAllAssets(Assets, true);
				UE_LOGFMT(LogPlainPropsEngine, Display, "Loading all {Assets} assets...", Assets.Num());
			}

			for (FAssetData& Map : Assets)
			{
				FSoftObjectPath AssetPath = Map.GetSoftObjectPath();
				AssetPath.LoadAsync({});
			}
		}
		else if (AssetString == TEXT("test"))
		{
			Objects.Append(CreateTestAssets());
		}
		else
		{
			int32 CommaIndex;
			while (AssetString.FindChar(',', CommaIndex))
			{
				FSoftObjectPath AssetPath(AssetString.Left(CommaIndex)); 
				UObject* Object = AssetPath.TryLoad();
				checkf(Object, TEXT("Failed to load %s (%s)"), *AssetPath.ToString(), *FString(AssetString.Left(CommaIndex)));
				Objects.Add(Object);
				AssetString.RightChopInline(CommaIndex + 1);
			}

			FSoftObjectPath AssetPath(AssetString);
			UObject* Object = AssetPath.TryLoad();
			checkf(Object, TEXT("Failed to load %s (%s)"), *AssetPath.ToString(), *FString(AssetString));
			Objects.Add(Object);
		}
	}

	FlushAsyncOperations(TEXT("InitialFlush"));

	UE_LOGFMT(LogPlainPropsEngine, Display, "Init worlds ...");
	InitWorlds();

	EBindMode Mode = EBindMode::All;
	if (Params.Find("-bind=source") != INDEX_NONE)
	{
		Mode = EBindMode::Source;
	}
	else if (Params.Find("-bind=runtime") != INDEX_NONE)
	{
		Mode = EBindMode::Runtime;
	}

	EBatchType BatchType = EBatchType::Plain;
	if (Params.Find("-batch=linker") != INDEX_NONE)
	{
		BatchType = EBatchType::Linker;
	}

	UE_LOGFMT(LogPlainPropsEngine, Display, "Binding types to PlainProps schemas...");
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BindAllTypes);
		SchemaBindAllTypes(Mode, BatchType);
		CustomBindEngineTypes(Mode);
		CustomBindAnimationTypes(Mode);
		CustomBindNiagaraTypes(Mode);
		CustomBindBlueprintTypes(Mode);
		CustomBindTestTypes(Mode);
	}

	// Default roundtrip options
	ERoundtrip Options = ERoundtrip::PP | ERoundtrip::UPS | ERoundtrip::TPS;

	// For linker batches always load PP with either one of TPS or UPS as a base
	if (BatchType == EBatchType::Linker)
	{
		Options = (Params.Find("-ups") != INDEX_NONE) ?
			ERoundtrip::PP | ERoundtrip::UPS :
			ERoundtrip::PP | ERoundtrip::TPS;
	}

	// Parse roundtrip options
	if (Params.Find("-pp") != INDEX_NONE)
	{
		Options = ERoundtrip::PP | ERoundtrip::TextMemory;
	}
	else if (Params.Find("-text") != INDEX_NONE)
	{
		Options = ERoundtrip::TextMemory | ERoundtrip::TextStable;
	}

	if (Params.Find("-json") != INDEX_NONE)
	{
		Options |= ERoundtrip::JSON;
	}

	UE_LOGFMT(LogPlainPropsEngine, Display, "Starting roundtrip test...");
	
	// Roundtrip
	if (BatchType == EBatchType::Linker)
	{
		RoundtripViaLinkerBatch(Objects, Options, LinkerDiffFilter, DropPackages);
	}
	else
	{ 
		RoundtripViaPlainBatch(Objects, Options);
	}
	
	TRACE_CPUPROFILER_EVENT_FLUSH();
	FPlatformMisc::RequestExit(true, TEXT("TestPlainProps"));
	UE_LOGFMT(LogPlainPropsEngine, Display, "Done!");
	return 0;
}
