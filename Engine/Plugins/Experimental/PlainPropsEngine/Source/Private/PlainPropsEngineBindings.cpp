// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsEngineBindings.h"
#include "PlainPropsUObjectRuntime.h"
#include "PlainPropsRoundtripTest.h" // EBindMode
#include "PlainPropsBind.h"
#include "PlainPropsLoad.h"
#include "PlainPropsSave.h"
#include "PlainPropsVisualize.h"
#include "Algo/Compare.h"
#include "Engine/Texture2D.h"
#include "GameplayEffect.h"
#include "Logging/StructuredLog.h"
#include "Misc/DefinePrivateMemberPtr.h"

DEFINE_LOG_CATEGORY(LogPlainPropsEngine);

// Temp hacks for non-intrusive prototype
UE_DEFINE_PRIVATE_MEMBER_PTR(FIntPoint, GTexture2DImportedSize, UTexture2D, ImportedSize);

////////////////////////////////////////////////////////////////////////////////////////////////

template <typename T>
struct TIsContiguousContainer<TIndirectArray<T>>
{
	enum { Value = true };
};

static bool DiffMip(const FTexture2DMipMap* A, const FTexture2DMipMap* B)
{
	unimplemented(); // todo
	return false;
}

// In :: since Algo::Compare calls UE::Core::Private:: instead of ::UE::Core::Private::,
// which fails to handle nested the PlainProps::UE namespace
static bool DiffMips(const TIndirectArray<FTexture2DMipMap>& A, const TIndirectArray<FTexture2DMipMap>& B)
{
	return !Algo::Compare(A, B, DiffMip);
}

namespace PlainProps::UE
{

static FEnumId			IndexEnum(FScopeId Scope, FAnsiStringView Name)			{ return GUE.Names.IndexEnum({Scope, GUE.Names.MakeTypename(Name)}); }
static FStructId		IndexStruct(FScopeId Scope, FAnsiStringView Name)		{ return GUE.Names.IndexStruct({Scope, GUE.Names.MakeTypename(Name)}); }
static FEnumId			IndexEngineEnum(FAnsiStringView Name)					{ return IndexEnum(GUE.Scopes.Engine, Name); }
static FStructId		IndexEngineStruct(FAnsiStringView Name)					{ return IndexStruct(GUE.Scopes.Engine, Name); }

////////////////////////////////////////////////////////////////////////////////////////////////

struct FTexture2DCook : ICustomBinding
{
	using Type = UTexture2D;

	const FEnumId TextureAddressId;
	const FDualStructId IntPointId;
	const FDualStructId PlatformTextureDataId;
	const FMemberId MemberIds[5];

	FTexture2DCook(TPropertySpecifier<5>& Spec)
	: TextureAddressId(IndexEngineEnum("TextureAddress"))
	, IntPointId(IndexStruct(GUE.Scopes.CoreUObject, "IntPoint"))
	, PlatformTextureDataId(IndexEngineStruct("PlatformTextureData"))
	, MemberIds{GUE.Names.NameMember("AddressX"),
				GUE.Names.NameMember("AddressY"),
				GUE.Names.NameMember("ImportedSize"),
				GUE.Names.NameMember("CookedFormats"),
				GUE.Names.NameMember("CookedTextures")}
	{
		Spec.Members[0] = Specify<TextureAddress>(TextureAddressId);
		Spec.Members[1] = Specify<TextureAddress>(TextureAddressId);
		Spec.Members[2] = FMemberSpec(IntPointId);
		Spec.Members[3] = DefaultRangeOf(Specify<EPixelFormat>(GUE.EnumIds.PixelFormat));
		Spec.Members[4] = DefaultRangeOf(PlatformTextureDataId);
	}
	
	inline void Save(FMemberBuilder& Dst, const UTexture2D& Src, const UTexture2D* Default, const FSaveContext& Ctx) const
	{
#if WITH_EDITOR // Todo: Encapsulate in a TScopedCookBinding, maybe use requires(WITH_EDTIOR)
		FTypedRange CookedTextures;
		FTypedRange CookedFormats = CookTexture(/* out */ CookedTextures, Src, Ctx);
		FIntPoint ImportedSize = Src.GetImportedSize();

		Dst.AddEnum(MemberIds[0], TextureAddressId, Src.AddressX.GetValue());
		Dst.AddEnum(MemberIds[1], TextureAddressId, Src.AddressY.GetValue());
		Dst.AddStruct(MemberIds[2], IntPointId, SaveStruct(&ImportedSize, IntPointId, Ctx));
		Dst.AddRange(MemberIds[3], CookedFormats);
		Dst.AddRange(MemberIds[4], CookedTextures);
#else
		unimplemented();
#endif // WITH_EDITOR
	}
		
	inline void Load(UTexture2D& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
	{
		check(Method == ECustomLoadMethod::Assign);
		FMemberLoader It(Src);
		
		Dst.AddressX = It.GrabLeaf().As<TextureAddress>();
		Dst.AddressY = It.GrabLeaf().As<TextureAddress>();
		LoadStruct(&(Dst.*GTexture2DImportedSize), It.GrabStruct());
		FLeafRangeLoadView CookedFormats = It.GrabRange().AsLeaves();
		FStructRangeLoadView CookedTextures = It.GrabRange().AsStructs();
		Dst.SetPlatformData(LoadTexture(CookedFormats, CookedTextures));
	}

	static bool Diff(const UTexture2D& A, const UTexture2D& B, const FBindContext& Ctx)
	{
		check(WITH_EDITOR);
		// Todo: Research how to implement this
		return A.GetPlatformData() != B.GetPlatformData();
	}

	static bool Diff(const UTexture2D& A, const UTexture2D& B, FDiffContext& Ctx)
	{
		return	A.GetTextureAddressX() != B.GetTextureAddressX() ||
				A.GetTextureAddressY() != B.GetTextureAddressY() ||
				A.GetImportedSize() != B.GetImportedSize() ||
				DiffTexture(*A.GetPlatformData(), *B.GetPlatformData(), Ctx);
	}
	
	FTypedRange CookTexture(FTypedRange& OutCookedTextures, const UTexture2D& Src, const FSaveContext& Ctx) const
	{
		unimplemented(); // todo
		return {};
	}
	
	FTexturePlatformData* LoadTexture(FLeafRangeLoadView CookedFormats, FStructRangeLoadView CookedTextures) const
	{
		unimplemented(); // todo
		return nullptr;
	}

	static bool DiffTexture(const FTexturePlatformData& A, const FTexturePlatformData& B, FDiffContext& Ctx)
	{
		return A.SizeX != B.SizeX || A.SizeY != B.SizeY || A.PackedData != B.PackedData || A.PixelFormat != B.PixelFormat ||
			A.OptData != B.OptData || DiffMips(A.Mips, B.Mips) || !!A.VTData != !!B.VTData;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FGameplayEffectVersionBinding : ICustomBinding
{
	using Type = FGameplayEffectVersion;
	using Ids = FRuntimeIds;

	const FEnumId GameplayEffectVersionId;
	const FMemberId MemberIds[1];
	
	FGameplayEffectVersionBinding(TPropertySpecifier<1>& Spec)
	: GameplayEffectVersionId(GUE.DynamicIds.GetEnum(StaticEnum<EGameplayEffectVersion>()))
	, MemberIds{Ids::IndexMember("CurrentVersion")}
	{
		Spec.Members[0] = Specify<EGameplayEffectVersion>(GameplayEffectVersionId);
	}

	void Save(FMemberBuilder& Dst, const FGameplayEffectVersion& Src, const FGameplayEffectVersion* Default, const FSaveContext& Ctx) const
	{
		Dst.AddEnum(MemberIds[0], GameplayEffectVersionId, Src.CurrentVersion);
	}

	inline static void Plan(FMemcpyLoadPlan& Out)
	{
		check(sizeof(FGameplayEffectVersion) == sizeof(EGameplayEffectVersion));
		Out.Size = sizeof(FGameplayEffectVersion);
	}
	
	inline static bool Diff(FGameplayEffectVersion A, FGameplayEffectVersion B, const FBindContext&)
	{
		// TODO: do we need support for opting out of delta serialization?
		// The existing custom serialize implementation always returns false from Identical as a hack to disable delta serialization.
		check(A.Identical(&B, /*PortFlags*/0) == false); // detect if the existing code is changed
		return A.CurrentVersion != B.CurrentVersion;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

void CustomBindEngineTypes(EBindMode Mode)
{
	{ DbgVis::FLoadStructPlan _; DbgVis::FIdVisualizer{}.KeepDebugInfo(&_); }

	// Todo: Ownership / memory leak
	new TScopedDefaultUStructBinding<FGameplayEffectVersion, FGameplayEffectVersionBinding>();

	if (Mode == EBindMode::Runtime)
	{
		new TScopedUClassBinding<UTexture2D, FDeltaRuntime, FTexture2DCook>();
	}
}

} // namespace PlainProps::UE

namespace PlainProps
{
template<> struct TOccupancyOf<FGameplayEffectVersion> : FRequireAll {};
} // namespace PlainProps
