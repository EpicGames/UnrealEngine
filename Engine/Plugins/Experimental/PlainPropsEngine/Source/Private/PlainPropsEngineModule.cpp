// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsEngineModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FPlainPropsEngineModule, PlainPropsEngine);

void FPlainPropsEngineModule::StartupModule() {}
void FPlainPropsEngineModule::ShutdownModule() {}