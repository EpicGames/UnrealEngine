// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsEngineTestUtils.h"
#include "PlainPropsEngineBindings.h"
#include "PlainPropsRoundtripTest.h"
#include "PlainPropsTestBindings.h"

namespace PlainProps::UE
{

void BindAllTypes(EBindMode Mode, EBatchType BatchType)
{
	SchemaBindAllTypes(Mode, BatchType);
	CustomBindEngineTypes(Mode);
	CustomBindAnimationTypes(Mode);
	CustomBindNiagaraTypes(Mode);
	CustomBindTestTypes(Mode);
}

} // namespace PlainProps::UE