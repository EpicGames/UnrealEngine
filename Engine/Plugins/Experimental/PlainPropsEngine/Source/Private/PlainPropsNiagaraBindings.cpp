// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsEngineBindings.h"
#include "PlainPropsUObjectRuntime.h"
#include "PlainPropsRoundtripTest.h" // EBindMode
#include "PlainPropsBind.h"
#include "PlainPropsLoad.h"
#include "PlainPropsSave.h"
#include "PlainPropsVisualize.h"
#include "PlainPropsInternalPrivateMemberPtr.h"
#include "NiagaraTypes.h"

// Temp hacks for non-intrusive prototype
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(FNiagaraTypeDefinition_Private)
PP_DEFINE_PRIVATE_MEMBER_PTR(FNiagaraTypeDefinition,		Flags,						uint8);	
PP_DEFINE_PRIVATE_MEMBER_PTR(FNiagaraTypeDefinition,		RegisteredTypeDefIndex,		int32);	
PP_DEFINE_PRIVATE_MEMBER_PTR(FNiagaraTypeDefinition,		GeneratePathNameHash,		void());	
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END()

PP_REFLECT_ENUM(FNiagaraTypeDefinition, FUnderlyingType, UT_None, UT_Class, UT_Struct, UT_Enum);
// The last two constants in FNiagaraTypeDefinition::FTypeFlags TF_SerializedAsLWC and TF_AllowLWC_DEPRECATED should be unused,
// leaving them out will cause a runtime assert if this is not the case.
PP_REFLECT_ENUM(FNiagaraTypeDefinition, FTypeFlags, TF_None, TF_Static);

////////////////////////////////////////////////////////////////////////////////////////////////

namespace PlainProps::UE::Niagara
{

// Custom bind FNiagaraTypeDefinition to modify the property values on Save and Load,
// could use a schema binding for actual serialization.
struct FNiagaraTypeDefinitionBinding : ICustomBinding
{
	using Type = FNiagaraTypeDefinition;
	using Ids = FRuntimeIds;
	using FUnderlyingType = FNiagaraTypeDefinition::FUnderlyingType;
	using FTypeFlags = FNiagaraTypeDefinition::FTypeFlags;

	const FEnumId UnderlyingTypeId;
	const FEnumId TypeFlagsId;
	const FMemberId MemberIds[3];
	
	FNiagaraTypeDefinitionBinding(TPropertySpecifier<3>& Spec)
	: UnderlyingTypeId(GetEnumId<Ids, FUnderlyingType>())
	, TypeFlagsId(GetEnumId<Ids, FTypeFlags>())
	, MemberIds{Ids::IndexMember("ClassStructOrEnum"), Ids::IndexMember("UnderlyingType"), Ids::IndexMember("Flags")}
	{
		Spec.Members[0] = GUE.Structs.ObjectPtr;
		Spec.Members[1] = Specify<FUnderlyingType>(UnderlyingTypeId);
		Spec.Members[2] = Specify<FTypeFlags>(TypeFlagsId);
	}

	void Save(FMemberBuilder& Dst, const FNiagaraTypeDefinition& Src, const FNiagaraTypeDefinition* Default, const FSaveContext& Ctx) const
	{
		using namespace FNiagaraTypeDefinition_Private;

		FBuiltStruct* ClassStructOrEnum = SaveStruct(&Src.ClassStructOrEnum, GUE.Structs.ObjectPtr, Ctx);
		FUnderlyingType UnderlyingType = static_cast<FUnderlyingType>(Src.UnderlyingType);
		FTypeFlags Flags = static_cast<FTypeFlags>(Src.GetFlags());

		Dst.AddStruct(MemberIds[0], GUE.Structs.ObjectPtr, ClassStructOrEnum);
		Dst.AddEnum(MemberIds[1], UnderlyingTypeId, UnderlyingType);
		Dst.AddEnum(MemberIds[2], TypeFlagsId, Flags);
	}

	void Load(FNiagaraTypeDefinition& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
	{
		using namespace FNiagaraTypeDefinition_Private;

		FMemberLoader It(Src);
		Dst.*_RegisteredTypeDefIndex = INDEX_NONE; // note: the private InvalidateCachedIndex() is not exported
		LoadStruct(&Dst.ClassStructOrEnum, It.GrabStruct());
		Dst.UnderlyingType = It.GrabLeaf().AsUnderlyingValue<uint16>();
		Dst.*_Flags = It.GrabLeaf().AsUnderlyingValue<uint8>();
		(Dst.*_GeneratePathNameHash)();
	}

	inline static bool Diff(FNiagaraTypeDefinition A, FNiagaraTypeDefinition B, const FBindContext&) { return A != B; }
};

// Custom bind FNiagaraTypeDefinitionHandle to Save and Load as a FNiagaraTypeDefinition
struct FNiagaraTypeDefinitionHandleBinding : ICustomBinding
{
	using Ids = FRuntimeIds;
	using Type = FNiagaraTypeDefinitionHandle;
	const FDualStructId NiagaraTypeDefinitionId;
	const FMemberId MemberIds[1];
	
	FNiagaraTypeDefinitionHandleBinding(TPropertySpecifier<1>& Spec)
	: NiagaraTypeDefinitionId(GUE.DynamicIds.GetStruct(FNiagaraTypeDefinition::StaticStruct()))
	, MemberIds{Ids::IndexMember("TypeDef")}
	{
		Spec.Members[0] = NiagaraTypeDefinitionId;
	}

	void Save(FMemberBuilder& Dst, const FNiagaraTypeDefinitionHandle& Src, const FNiagaraTypeDefinitionHandle* Default, const FSaveContext& Ctx) const
	{
		FNiagaraTypeDefinition TypeDef = *Src;
		FBuiltStruct* BuiltStruct = SaveStruct(&TypeDef, NiagaraTypeDefinitionId, Ctx);
		Dst.AddStruct(MemberIds[0], NiagaraTypeDefinitionId, BuiltStruct);
	}

	void Load(FNiagaraTypeDefinitionHandle& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
	{
		FNiagaraTypeDefinition TypeDef;
		LoadSoleStruct(&TypeDef, Src);
		Dst = FNiagaraTypeDefinitionHandle(TypeDef);
	}

	inline static bool Diff(FNiagaraTypeDefinitionHandle A, FNiagaraTypeDefinitionHandle B, const FBindContext&) { return A != B; }
};

} // namespace PlainProps::UE::Niagara

////////////////////////////////////////////////////////////////////////////////////////////////

namespace PlainProps::UE
{
void CustomBindNiagaraTypes(EBindMode Mode)
{
	// Todo: Ownership / memory leak
	new TScopedDefaultEnumDeclaration<FNiagaraTypeDefinition::FUnderlyingType, EEnumMode::Flat>();	// global scope
	new TScopedDefaultEnumDeclaration<FNiagaraTypeDefinition::FTypeFlags, EEnumMode::Flag>();		// global scope
	new TScopedDefaultUStructBinding<FNiagaraTypeDefinition, Niagara::FNiagaraTypeDefinitionBinding>();
	new TScopedDefaultUStructBinding<FNiagaraTypeDefinitionHandle, Niagara::FNiagaraTypeDefinitionHandleBinding>();
}
} // namespace PlainProps::UE

////////////////////////////////////////////////////////////////////////////////////////////////
namespace PlainProps
{
template<> struct TOccupancyOf<FNiagaraTypeDefinition> : FRequireAll {};
template<> struct TOccupancyOf<FNiagaraTypeDefinitionHandle> : FRequireAll {};

// todo: Optionally add a fake "/Script/Niagara" scope to these non-reflected class internal native enums?
// template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FNiagaraTypeDefinition::FUnderlyingType>>()	{ return UE::Niagara::GetNiagaraScope(); }
// template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FNiagaraTypeDefinition::FTypeFlags>>()		{ return UE::Niagara::GetNiagaraScope(); }
} // namespace PlainProps
