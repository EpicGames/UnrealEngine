// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsTestAssets.h"

#if WITH_TESTS && !defined(PLATFORM_COMPILER_IWYU)

#include "PlainPropsTestTypes.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/PackagePath.h"
#include "Serialization/LargeMemoryReader.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectGlobalsInternal.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectThreadContext.h"

namespace PlainProps::UE
{

template<typename T>
T* CreateTestAsset(const FString& AssetName, T* Template = nullptr)
{
	const FString PackageName = TEXT("/Test/") + AssetName;
	UPackage* Package = CreatePackage(*PackageName);

	T* Asset = NewObject<T>(Package, T::StaticClass(), *AssetName, RF_Public | RF_Standalone, Template);
	check(Asset);
	FAssetRegistryModule::AssetCreated(Asset);

	Package->ClearPackageFlags(PKG_NewlyCreated);
	Package->SetFlags(RF_Public | RF_WasLoaded);

	return Asset;
}

TArray<UObject*> CreateTestAssets()
{
	TArray<UObject*> Assets;

	{
		UPlainPropsInstancedStructTestAsset* TestAsset = CreateTestAsset<UPlainPropsInstancedStructTestAsset>(TEXT("PlainPropsInstancedStructTestAssetModified"));
		{
			FPlainPropsInstancedStructTestSchemaA& InstancedStruct = TestAsset->StructA.GetMutable<FPlainPropsInstancedStructTestSchemaA>();
			InstancedStruct.bValue = true;
			InstancedStruct.X++;
		}
		
		{
			FPlainPropsInstancedStructTestSchemaB& InstancedStruct = TestAsset->StructB.GetMutable<FPlainPropsInstancedStructTestSchemaB>();
			InstancedStruct.Count++;
			InstancedStruct.Label = FName("Modified");
		}

		{
			FPlainPropsInstancedStructTestCustomBound& InstancedStruct = TestAsset->StructC.GetMutable<FPlainPropsInstancedStructTestCustomBound>();
			InstancedStruct.bValue = true;
			InstancedStruct.Id++;
		}

		{
			FPlainPropsInstancedStructTestSchemaA& InstancedStruct = TestAsset->MixedArray[0].GetMutable<FPlainPropsInstancedStructTestSchemaA>();
			InstancedStruct.X+=1;
			InstancedStruct.Y+=2;
			InstancedStruct.Z+=3;
		}
		
		{
			FPlainPropsInstancedStructTestSchemaB& InstancedStruct = TestAsset->MixedArray[1].GetMutable<FPlainPropsInstancedStructTestSchemaB>();
			InstancedStruct = FPlainPropsInstancedStructTestSchemaB();

			InstancedStruct.Label = FName("Modified2");
		}
		
		{
			FPlainPropsInstancedStructTestCustomBound& InstancedStruct = TestAsset->MixedArray[2].GetMutable<FPlainPropsInstancedStructTestCustomBound>();
			InstancedStruct.bValue = true;
			InstancedStruct.Id++;
		}

		{
			TestAsset->MixedArray[3].InitializeAs<FPlainPropsInstancedStructTestSchemaB>();
			FPlainPropsInstancedStructTestSchemaB& InstancedStruct = TestAsset->MixedArray[3].GetMutable<FPlainPropsInstancedStructTestSchemaB>();
			InstancedStruct.Label = FName("Modified3");
		}

		{
			TestAsset->MixedArray.Add(FInstancedStruct::Make<FPlainPropsInstancedStructTestSchemaA>(FPlainPropsInstancedStructTestSchemaA({ {true}, 20.f, 21.f, 22.f })));
		}
		
		{
			FPlainPropsInstancedStructTestSchemaA& InstancedStruct = TestAsset->BaseStruct.GetMutable<FPlainPropsInstancedStructTestSchemaA>();
			InstancedStruct.X+=2;
			InstancedStruct.Y+=3;
			InstancedStruct.Z+=4;
		}

		{
			TestAsset->EmptyStruct.InitializeAs<FPlainPropsInstancedStructTestCustomBound>(FPlainPropsInstancedStructTestCustomBound{ { true }, 12 });
		}

		Assets.Add(TestAsset);
	}

	return Assets;
}

} // namespace PlainProps::UE

#else

namespace PlainProps::UE
{

TArray<UObject*> CreateTestAssets()
{
	return {};
}

}

#endif // WITH_TESTS && !defined(PLATFORM_COMPILER_IWYU)