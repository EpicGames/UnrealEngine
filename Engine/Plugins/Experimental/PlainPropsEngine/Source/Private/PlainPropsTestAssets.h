// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

class UObject;

namespace PlainProps::UE
{

TArray<UObject*> CreateTestAssets();

} // namespace PlainProps::UE
