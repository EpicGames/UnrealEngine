// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsTestBindings.h"

#if WITH_TESTS && !defined(PLATFORM_COMPILER_IWYU)

#include "PlainPropsTestTypes.h"
#include "PlainPropsUObjectRuntime.h"

namespace PlainProps::UE
{

namespace PlainPropsTestWithCustomBinding
{

/** 
 *  Custom PlainProps binding for FPlainPropsInstancedStructTestCustomBound — serializes members explicitly by name.
 *  Used to exercise the FInstancedStruct binding when the inner struct itself has a custom binding. 
 * 
 *  todo: once custom binding can forward to schema simplify this implementation to avoid having to keep it in sync to avoid diffs. 
 */
struct FCustomBinding : ICustomBinding
{
	using Type = FPlainPropsInstancedStructTestCustomBound;
	
	static constexpr int32 MemberCount = 2;
	const FMemberId MemberIds[MemberCount];

	template<class Ids>
	FCustomBinding(TCustomSpecifier<Ids, MemberCount>& Spec)
	: MemberIds{Ids::IndexMember("bValue"), Ids::IndexMember("Id")}
	{
		Spec.Members[0] = Specify<bool>();
		Spec.Members[1] = Specify<int32>();
	}

	void Save(FMemberBuilder& Dst, const Type& Src, const Type* /*Default*/, const FSaveContext&) const
	{
		Dst.Add(MemberIds[0], Src.bValue);
		Dst.Add(MemberIds[1], Src.Id);
	}

	void Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod /*Method*/) const
	{
		FMemberLoader Members(Src);
		Dst.bValue = Members.GrabLeaf().AsBool();
		Dst.Id = Members.GrabLeaf().AsS32();
	}

	static bool Diff(const Type& A, const Type& B, const FBindContext&)
	{
		return !(A == B);
	}
};

} // namespace PlainPropsTestWithCustomBinding

void CustomBindTestTypes(EBindMode Mode)
{
	// Todo: Ownership / memory leak
	new TScopedDefaultUStructBinding<FPlainPropsInstancedStructTestCustomBound, PlainPropsTestWithCustomBinding::FCustomBinding>();
}

} // namespace PlainProps::UE

#else

namespace PlainProps::UE
{

void CustomBindTestTypes(EBindMode)
{
}

} // namespace PlainProps::UE

#endif // WITH_TESTS && !defined(PLATFORM_COMPILER_IWYU)