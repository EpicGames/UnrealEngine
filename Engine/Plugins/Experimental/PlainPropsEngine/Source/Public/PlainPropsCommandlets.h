// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "PlainPropsCommandlets.generated.h"

UCLASS()
class UTestPlainPropsCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()
	virtual int32 Main(const FString& Params) override;
};
