// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class FPlainPropsEngineModule final : public IModuleInterface
{
	virtual void StartupModule();
	virtual void ShutdownModule();
};