// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TESTS && !defined(PLATFORM_COMPILER_IWYU)

#include "Engine/DataAsset.h"
#include "StructUtils/InstancedStruct.h"

#include "PlainPropsTestTypes.generated.h"

USTRUCT()
struct FPlainPropsInstancedStructTestSchemaBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Default")
	bool bValue = false;
};

USTRUCT()
struct FPlainPropsInstancedStructTestSchemaA : public FPlainPropsInstancedStructTestSchemaBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Default")
	float X = 1.f;

	UPROPERTY(EditAnywhere, Category = "Default")
	float Y = 2.f;

	UPROPERTY(EditAnywhere, Category = "Default")
	float Z = 3.f;
};

USTRUCT()
struct FPlainPropsInstancedStructTestSchemaB : public FPlainPropsInstancedStructTestSchemaBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Default")
	int32 Count = 4;

	UPROPERTY(EditAnywhere, Category = "Default")
	FName Label = FName("Five");
};

USTRUCT()
struct FPlainPropsInstancedStructTestCustomBound : public FPlainPropsInstancedStructTestSchemaBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Default")
	int32 Id = 6;

	// Used by custom binding
	friend bool operator==(const FPlainPropsInstancedStructTestCustomBound& Lhs, const FPlainPropsInstancedStructTestCustomBound& Rhs)
	{
		return Lhs.Id == Rhs.Id && Lhs.bValue == Rhs.bValue;
	}
};

/** Test asset for exercising FInstancedStruct serialization through the PlainProps custom binding. */
UCLASS()
class UPlainPropsInstancedStructTestAsset : public UDataAsset
{
	GENERATED_BODY()

public:
	UPlainPropsInstancedStructTestAsset()
	{
		StructA.InitializeAs<FPlainPropsInstancedStructTestSchemaA>();
		StructB.InitializeAs<FPlainPropsInstancedStructTestSchemaB>();
		StructC.InitializeAs<FPlainPropsInstancedStructTestCustomBound>();

		MixedArray.Add(FInstancedStruct::Make<FPlainPropsInstancedStructTestSchemaA>(FPlainPropsInstancedStructTestSchemaA({{true}, 7.f, 8.f, 9.f})));
		MixedArray.Add(FInstancedStruct::Make<FPlainPropsInstancedStructTestSchemaB>(FPlainPropsInstancedStructTestSchemaB{{false}, 10, FName("Eleven")}));
		MixedArray.Add(FInstancedStruct::Make<FPlainPropsInstancedStructTestCustomBound>(FPlainPropsInstancedStructTestCustomBound{{true}, 12}));
		MixedArray.Add(FInstancedStruct::Make<FPlainPropsInstancedStructTestSchemaA>());

		// EmptyStruct intentionally left empty

		BaseStruct.InitializeAs<FPlainPropsInstancedStructTestSchemaA>(FPlainPropsInstancedStructTestSchemaA{ { false }, 13.f, 14.f, 15.f });
	}

	// Single FInstancedStruct holding FPlainPropsInstancedStructTestSchemaA
	UPROPERTY(EditAnywhere, Category = "Default")
	FInstancedStruct StructA;

	// Single FInstancedStruct holding FPlainPropsInstancedStructTestSchemaB
	UPROPERTY(EditAnywhere, Category = "Default")
	FInstancedStruct StructB;

	// Single FInstancedStruct holding FPlainPropsInstancedStructTestCustomBound (with a custom binding)
	UPROPERTY(EditAnywhere, Category = "Default")
	FInstancedStruct StructC;

	// Heterogeneous array: mix of FPlainPropsInstancedStructTestSchemaA, FPlainPropsInstancedStructTestSchemaB and FPlainPropsInstancedStructTestCustomBound entries
	UPROPERTY(EditAnywhere, Category = "Default")
	TArray<FInstancedStruct> MixedArray;

	// Intentionally empty — exercises the null/empty case
	UPROPERTY(EditAnywhere, Category = "Default")
	FInstancedStruct EmptyStruct;

	// Single TInstancedStruct<FPlainPropsInstancedStructTestSchemaBase> holding FPlainPropsInstancedStructTestSchemaA
	UPROPERTY(EditAnywhere, Category = "Default")
	TInstancedStruct<FPlainPropsInstancedStructTestSchemaBase> BaseStruct;
};

#endif // WITH_TESTS && !defined(PLATFORM_COMPILER_IWYU)